// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteDatabaseConnection.h"

#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Misc/ByteSwap.h"

/**
 * Sends a command to the database proxy.
 *
 * @param	Cmd		The command to be sent.
 */
bool ExecuteDBProxyCommand(FSocket *Socket, const FString& Cmd)
{
	check(Socket);

	const int32 CmdStrLength = Cmd.Len() + 1; // add 1 so we also send NULL

	// convert to network byte ordering. This is important for running on the ps3 and xenon
	TCHAR *SendBuf = (TCHAR*)FMemory::Malloc(CmdStrLength * sizeof(TCHAR));
	FMemory::Memcpy(SendBuf, *Cmd, CmdStrLength * sizeof(TCHAR));
	NETWORK_ORDER_TCHARARRAY(SendBuf);

	int32 BytesSent = 0;
	bool bRet = Socket->Send((uint8*)SendBuf, CmdStrLength * sizeof(TCHAR), BytesSent);

	FMemory::Free(SendBuf);

	return bRet;
}

////////////////////////////////////////////// FRemoteDatabaseConnection ///////////////////////////////////////////////////////
/**
 * Constructor.
 */
FRemoteDatabaseConnection::FRemoteDatabaseConnection()
: Socket(nullptr)
{
}

/**
 * Destructor.
 */
FRemoteDatabaseConnection::~FRemoteDatabaseConnection()
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	check(SocketSubsystem);

	if (Socket)
	{
		SocketSubsystem->DestroySocket(Socket);
		Socket = nullptr;
	}
}

/**
 * Opens a connection to the database.
 *
 * @param	ConnectionString	Connection string passed to database layer
 * @param   RemoteConnectionIP  The IP address which the RemoteConnection should connect to
 * @param   RemoteConnectionStringOverride  The connection string which the RemoteConnection is going to utilize
 *
 * @return	true if connection was successfully established, false otherwise
 */
bool FRemoteDatabaseConnection::Open( const TCHAR* ConnectionString, const TCHAR* RemoteConnectionIP, const TCHAR* RemoteConnectionStringOverride )
{
	bool bIsValid = false;
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	check(SocketSubsystem);

	TSharedRef<FInternetAddr> Address = SocketSubsystem->CreateInternetAddr();
	Address->SetIp(RemoteConnectionIP, bIsValid);
	Address->SetPort(10500);

	// The socket won't work if secure connections are enabled, so don't try
	if (SocketSubsystem->RequiresEncryptedPackets() == false && Socket == nullptr)
	{
		Socket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("remote database connection"), Address->GetProtocolType());
	}
	else
	{
		bIsValid = false;
	}

	if (bIsValid && Socket)
	{
		bIsValid = Socket->Connect(*Address);

		if (bIsValid && RemoteConnectionStringOverride)
		{
			SetConnectionString(RemoteConnectionStringOverride);
		}
	}
	return bIsValid;
}

/**
 * Closes connection to database.
 */
void FRemoteDatabaseConnection::Close()
{
	if ( Socket )
	{
		Socket->Close();
	}
}

/**
 * Executes the passed in command on the database.
 *
 * @param CommandString		Command to execute
 *
 * @return true if execution was successful, false otherwise
 */
bool FRemoteDatabaseConnection::Execute(const TCHAR* CommandString)
{
	return ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<command results=\"false\">%s</command>"), CommandString));
}

/**
 * Executes the passed in command on the database. The caller is responsible for deleting
 * the created RecordSet.
 *
 * @param CommandString		Command to execute
 * @param RecordSet			Reference to recordset pointer that is going to hold result
 *
 * @return true if execution was successful, false otherwise
 */
bool FRemoteDatabaseConnection::Execute(const TCHAR* CommandString, FDataBaseRecordSet*& RecordSet)
{
	RecordSet = NULL;
	bool bRetVal = ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<command results=\"true\">%s</command>"), CommandString));
	int32 ResultID = 0;
	int32 BytesRead;

	if(bRetVal)
	{
		Socket->Recv((uint8*)&ResultID, sizeof(int32), BytesRead);
		bRetVal = BytesRead == sizeof(int32);

		if(bRetVal)
		{
			RecordSet = new FRemoteDataBaseRecordSet(ResultID, Socket);
		}
	}

	return bRetVal;
}

/**
 * Sets the connection string to be used for this connection in the DB proxy.
 *
 * @param	ConnectionString	The new connection string to use.
 */
bool FRemoteDatabaseConnection::SetConnectionString(const TCHAR* ConnectionString)
{
	return ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<connectionString>%s</connectionString>"), ConnectionString));
}

////////////////////////////////////////////// FRemoteDataBaseRecordSet ///////////////////////////////////////////////////////

/** Moves to the first record in the set. */
void FRemoteDataBaseRecordSet::MoveToFirst()
{
	ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<movetofirst resultset=\"%s\"/>"), *ID));
}

/** Moves to the next record in the set. */
void FRemoteDataBaseRecordSet::MoveToNext()
{
	ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<movetonext resultset=\"%s\"/>"), *ID));
}

/**
 * Returns whether we are at the end.
 *
 * @return true if at the end, false otherwise
 */
bool FRemoteDataBaseRecordSet::IsAtEnd() const
{
	ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<isatend resultset=\"%s\"/>"), *ID));

	int32 BytesRead;
	bool bResult;
	Socket->Recv((uint8*)&bResult, sizeof(bool), BytesRead);

	if(BytesRead != sizeof(bool))
	{
		bResult = false;
	}

	return bResult;
}

/**
 * Returns a string associated with the passed in field/ column for the current row.
 *
 * @param	Column	Name of column to retrieve data for in current row
 */
FString FRemoteDataBaseRecordSet::GetString( const TCHAR* Column ) const
{
	ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<getstring resultset=\"%s\">%s</getstring>"), *ID, Column));

	const int32 BUFSIZE = 2048;
	int32 BytesRead;
	int32 StrLength;
	TCHAR Buf[BUFSIZE];

	Socket->Recv((uint8*)&StrLength, sizeof(int32), BytesRead);

	StrLength = NETWORK_ORDER32(StrLength);

	if(BytesRead != sizeof(int32) || StrLength <= 0)
	{
		return TEXT("");
	}

	if(StrLength > BUFSIZE - 1)
	{
		StrLength = BUFSIZE - 1;
	}

	Socket->Recv((uint8*)Buf, StrLength * sizeof(TCHAR), BytesRead);

	// TCHAR is assumed to be wchar_t so if we recv an odd # of bytes something messed up occurred. Round down to the nearest wchar_t and then convert to number of TCHAR's.
	BytesRead -= BytesRead & 1; // rounding down
	BytesRead >>= 1; // conversion
	Buf[BytesRead - 1] = 0; // ensure null terminator

	// convert from network to host byte order
	NETWORK_ORDER_TCHARARRAY(Buf);

	return FString(Buf);
}

/**
* Returns an integer associated with the passed in field/ column for the current row.
*
* @param	Column	Name of column to retrieve data for in current row
*/
int32 FRemoteDataBaseRecordSet::GetInt( const TCHAR* Column ) const
{
	ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<getint resultset=\"%s\">%s</getint>"), *ID, Column));

	int32 BytesRead;
	int32 Value;

	Socket->Recv((uint8*)&Value, sizeof(int32), BytesRead);

	return NETWORK_ORDER32(Value);
}

/**
* Returns a float associated with the passed in field/ column for the current row.
*
* @param	Column	Name of column to retrieve data for in current row
*/
float FRemoteDataBaseRecordSet::GetFloat( const TCHAR* Column ) const
{
	ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<getfloat resultset=\"%s\">%s</getfloat>"), *ID, Column));

	int32 BytesRead;
	int32 Temp;

	Socket->Recv((uint8*)&Temp, sizeof(int32), BytesRead);

	Temp = NETWORK_ORDER32(Temp);

	float Value = *((float*)&Temp);

	return Value;
}

/** Constructor. */
FRemoteDataBaseRecordSet::FRemoteDataBaseRecordSet(int32 ResultSetID, FSocket *Connection) : Socket(nullptr)
{
	check(ResultSetID >= 0);
	check(Connection);

	// NOTE: This socket will be deleted by whatever created it (prob an FRemoteDatabaseConnection), not this class.
	Socket = Connection;
	ID = FString::Printf(TEXT("%i"), ResultSetID);
}

/** Virtual destructor as class has virtual functions. */
FRemoteDataBaseRecordSet::~FRemoteDataBaseRecordSet()
{
	// tell the DB proxy to clean up the resources allocated for the result set.
	ExecuteDBProxyCommand(Socket, FString::Printf(TEXT("<closeresultset resultset=\"%s\"/>"), *ID));
}
