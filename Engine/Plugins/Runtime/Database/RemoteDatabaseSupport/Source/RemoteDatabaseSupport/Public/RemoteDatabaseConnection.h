// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Database.h"

//forward declarations
class FSocket;

/**
* This class allows for connections to a remote database proxy that allows any platform, regardless of native DB support, to talk to a DB.
*/
class REMOTEDATABASESUPPORT_API FRemoteDatabaseConnection : public FDataBaseConnection
{
private:
	FSocket *Socket;

public:
	/**
	 * Constructor.
	 */
	FRemoteDatabaseConnection();

	/**
	 * Destructor.
	 */
	virtual ~FRemoteDatabaseConnection();

	/**
	 * Opens a connection to the database.
	 *
	 * @param	ConnectionString	Connection string passed to database layer
	 * @param   RemoteConnectionIP  The IP address which the RemoteConnection should connect to
	 * @param   RemoteConnectionStringOverride  The connection string which the RemoteConnection is going to utilize
	 *
	 * @return	true if connection was successfully established, false otherwise
	 */
	virtual bool Open( const TCHAR* ConnectionString, const TCHAR* RemoteConnectionIP, const TCHAR* RemoteConnectionStringOverride );

	/**
	* Closes connection to database.
	*/
	virtual void Close();

	/**
	* Executes the passed in command on the database.
	*
	* @param CommandString		Command to execute
	*
	* @return true if execution was successful, false otherwise
	*/
	virtual bool Execute(const TCHAR* CommandString);

	/**
	 * Executes the passed in command on the database. The caller is responsible for deleting
	 * the created RecordSet.
	 *
	 * @param CommandString		Command to execute
	 * @param RecordSet			Reference to recordset pointer that is going to hold result
	 *
	 * @return true if execution was successful, false otherwise
	 */
	virtual bool Execute(const TCHAR* CommandString, FDataBaseRecordSet*& RecordSet);

	/**
	 * Sets the connection string to be used for this connection in the DB proxy.
	 *
	 * @param	ConnectionString	The new connection string to use.
	 */
	bool SetConnectionString(const TCHAR* ConnectionString);
};

/**
 * A record set that is accessed from a DB proxy.
 */
class REMOTEDATABASESUPPORT_API FRemoteDataBaseRecordSet : public FDataBaseRecordSet
{
private:
	/** The record set's ID within the DB proxy. */
	FString ID;

	/** The connection to the proxy DB */
	FSocket *Socket;

	// Protected functions used internally for iteration.
protected:
	/** Moves to the first record in the set. */
	virtual void MoveToFirst();

	/** Moves to the next record in the set. */
	virtual void MoveToNext();

	/**
	 * Returns whether we are at the end.
	 *
	 * @return true if at the end, false otherwise
	 */
	virtual bool IsAtEnd() const;

public:
	/**
	 * Returns a string associated with the passed in field/ column for the current row.
	 *
	 * @param	Column	Name of column to retrieve data for in current row
	 */
	virtual FString GetString( const TCHAR* Column ) const;

	/**
	 * Returns an integer associated with the passed in field/ column for the current row.
	 *
	 * @param	Column	Name of column to retrieve data for in current row
	 */
	virtual int32 GetInt( const TCHAR* Column ) const;

	/**
	 * Returns a float associated with the passed in field/ column for the current row.
	 *
	 * @param	Column	Name of column to retrieve data for in current row
	 */
	virtual float GetFloat( const TCHAR* Column ) const;

	/** Constructor. */
	FRemoteDataBaseRecordSet(int32 ResultSetID, FSocket *Connection);

	/** Virtual destructor as class has virtual functions. */
	virtual ~FRemoteDataBaseRecordSet();
};
